// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class CurveAssetEditor : ModuleRules
{
	public CurveAssetEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("LevelEditor");
        PublicIncludePathModuleNames.Add("WorkspaceMenuStructure");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"CurveEditor",
				"Engine",
				"Slate",
				"SlateCore",
				"EditorFramework",
				"UnrealEd",
				"InputCore",
				"TimeManagement",
			}
		);

		DynamicallyLoadedModuleNames.Add("WorkspaceMenuStructure");
	}
}
