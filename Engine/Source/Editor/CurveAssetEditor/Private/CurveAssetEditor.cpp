// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveAssetEditor.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreGlobals.h"
#include "CurveAssetEditorModule.h"
#include "CurveEditor.h"
#include "CurveEditorCommands.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"
#include "Curves/CurveBase.h"
#include "Curves/CurveLinearColor.h"
#include "Curves/RichCurve.h"
#include "Delegates/Delegate.h"
#include "DetailsViewArgs.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "ICurveEditorBounds.h"
#include "ICurveEditorModule.h"
#include "IDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "RichCurveEditorModel.h"
#include "SColorGradientCurveEditorView.h"
#include "SColorGradientEditor.h"
#include "SCurveEditorPanel.h"
#include "SlotBase.h"
#include "Modification/Resolution/CurveMetaDataIdentifiers.h"
#include "Modification/Resolution/CurveModelLookUpInfo.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Textures/SlateIcon.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Trace/Detail/Channel.h"
#include "Tree/CurveEditorTree.h"
#include "Tree/ICurveEditorTreeItem.h"
#include "Tree/SCurveEditorTree.h"
#include "Tree/SCurveEditorTreePin.h"
#include "Tree/SCurveEditorTreeSelect.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class FExtender;
class ITableRow;
class SWidget;

#define LOCTEXT_NAMESPACE "CurveAssetEditor"

const FName FCurveAssetEditor::CurveTabId( TEXT( "CurveAssetEditor_Curve" ) );
const FName FCurveAssetEditor::ColorCurveEditorTabId(TEXT("CurveAssetEditor_ColorCurveEditor"));

struct FCurveAssetEditorTreeItem : public ICurveEditorTreeItem
{
	FCurveAssetEditorTreeItem(TWeakObjectPtr<UCurveBase> InCurveOwner, const FRichCurveEditInfo& InEditInfo)
		: CurveOwner(InCurveOwner)
		, EditInfo(InEditInfo)
	{
		if (CurveOwner.IsValid())
		{
			CurveName = FText::FromName(EditInfo.CurveName);
			CurveColor = CurveOwner->GetCurveColor(EditInfo);
		}
	}

	virtual TSharedPtr<SWidget> GenerateCurveEditorTreeWidget(const FName& InColumnName, TWeakPtr<FCurveEditor> InCurveEditor, FCurveEditorTreeItemID InTreeItemID, const TSharedRef<ITableRow>& TableRow) override
	{
		if (InColumnName == ColumnNames.Label)
		{
			return SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.Padding(FMargin(4.f))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(CurveName)
				.ColorAndOpacity(FSlateColor(CurveColor))
			];
		}
		else if (InColumnName == ColumnNames.SelectHeader)
		{
			return SNew(SCurveEditorTreeSelect, InCurveEditor, InTreeItemID, TableRow);
		}
		else if (InColumnName == ColumnNames.PinHeader)
		{
			return SNew(SCurveEditorTreePin, InCurveEditor, InTreeItemID, TableRow);
		}

		return nullptr;
	}

	virtual void CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels) override
	{
		if (!CurveOwner.IsValid())
		{
			return;
		}

		TUniquePtr<FRichCurveEditorModelRaw> NewCurve = MakeUnique<FRichCurveEditorModelRaw>(static_cast<FRichCurve*>(EditInfo.CurveToEdit), CurveOwner.Get());
		NewCurve->SetShortDisplayName(CurveName);
		NewCurve->SetChannelName(EditInfo.CurveName);
		NewCurve->SetColor(CurveColor);
		OutCurveModels.Add(MoveTemp(NewCurve));
	}
	
private:
	TWeakObjectPtr<UCurveBase> CurveOwner;
	FRichCurveEditInfo EditInfo;
	FText CurveName;
	FLinearColor CurveColor;
};

namespace UE::CurveAssetEditor
{
/**
 * Tries to create a FCurveModel that was previously in the FCurveEditor but has since been removed.
 * This is needed for Curve Editor's command based undo system.
 * 
 * The user may make a change to curve X, then click on curve Y, and then undo; FCurveEditor::CurveData only contains the visible curves, so it will
 * ask us to temporarily recreate the curve for X, which we try to match based off of the metadata it had when it existed.
 */
static TUniquePtr<FCurveModel> ResolveCurveModel(const CurveEditor::FCurveModelLookUpArgs& InArgs)
{
	const CurveEditor::FCurveMetaDataIdentifiers* Metadata = InArgs.GetCurveMetaData();
	if (!Metadata)
	{
		return nullptr;
	}
	
	UCurveBase* CurveOwner = Cast<UCurveBase>(Metadata->Owner.Get());
	if (!CurveOwner)
	{
		return nullptr;
	}

	TArray<FRichCurveEditInfo> CurveInfos = CurveOwner->GetCurves();
	if (CurveInfos.Num() == 1)
	{
		return MakeUnique<FRichCurveEditorModelRaw>(static_cast<FRichCurve*>(CurveInfos[0].CurveToEdit), CurveOwner);
	}
	const FRichCurveEditInfo* Data = CurveInfos.FindByPredicate([Metadata](const FRichCurveEditInfo& Data)
	{
		return Data.CurveName == Metadata->ChannelName;
	});
	return Data ? MakeUnique<FRichCurveEditorModelRaw>(static_cast<FRichCurve*>(Data->CurveToEdit), CurveOwner) : nullptr;
}
}


void FCurveAssetEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_CurveAssetEditor", "Curve Asset Editor"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner( CurveTabId, FOnSpawnTab::CreateSP(this, &FCurveAssetEditor::SpawnTab_CurveAsset) )
		.SetDisplayName( LOCTEXT("CurveTab", "Curve") )
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.CurveBase"));

	if (ColorCurveDetailsView)
	{
		InTabManager->RegisterTabSpawner(ColorCurveEditorTabId, FOnSpawnTab::CreateSP(this, &FCurveAssetEditor::SpawnTab_ColorCurveEditor))
			.SetDisplayName(LOCTEXT("ColorCurveEditorTab", "Color Curve Editor"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.CurveBase"));
	}
}

void FCurveAssetEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner( CurveTabId );
	InTabManager->UnregisterTabSpawner(ColorCurveEditorTabId);
}

void FCurveAssetEditor::InitCurveAssetEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UCurveBase* CurveToEdit )
{	

	TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_CurveAssetEditor_Layout_v2")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.9f)
				->SetHideTabWell(true)
				->AddTab(CurveTabId, ETabState::OpenedTab)
			)
		);

	UCurveLinearColor* ColorCurve = Cast<UCurveLinearColor>(CurveToEdit);
	if (ColorCurve)
	{

		StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_CurveAssetEditor_Layout_ColorCurvev3")
			->AddArea
			(
				FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Horizontal)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.8f)
						->SetHideTabWell(true)
						->AddTab(CurveTabId, ETabState::OpenedTab)
					)

					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.2f)
						->SetHideTabWell(true)
						->AddTab(ColorCurveEditorTabId, ETabState::OpenedTab)
					)
				)
			);

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

		ColorCurveDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	}
	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	const bool bToolbarFocusable = false;
	const bool bUseSmallIcons = true;
	FAssetEditorToolkit::InitAssetEditor( Mode, InitToolkitHost, FCurveAssetEditorModule::CurveAssetEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, CurveToEdit, bToolbarFocusable, bUseSmallIcons);
	
	FCurveAssetEditorModule& CurveAssetEditorModule = FModuleManager::LoadModuleChecked<FCurveAssetEditorModule>( "CurveAssetEditor" );
	AddMenuExtender(CurveAssetEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
	AddToolbarExtender(GetToolbarExtender());

	// @todo toolkit world centric editing
	/*// Setup our tool's layout
	if( IsWorldCentricAssetEditor() )
	{
		const FString TabInitializationPayload(TEXT(""));		// NOTE: Payload not currently used for table properties
		SpawnToolkitTab( CurveTabId, TabInitializationPayload, EToolkitTabSpot::Details );
	}*/

	if (CurveEditor.IsValid())
	{
		RegenerateMenusAndToolbars();
	}

	if (ColorCurve)
	{
		ColorCurveDetailsView->SetObject(ColorCurve);
	}
}

FName FCurveAssetEditor::GetToolkitFName() const
{
	return FName("CurveAssetEditor");
}

FText FCurveAssetEditor::GetBaseToolkitName() const
{
	return LOCTEXT( "AppLabel", "Curve Asset Editor" );
}

FString FCurveAssetEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "CurveAsset ").ToString();
}

FLinearColor FCurveAssetEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.0f, 0.0f, 0.2f, 0.5f );
}

TSharedRef<SDockTab> FCurveAssetEditor::SpawnTab_CurveAsset( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId().TabType == CurveTabId );


	CurveEditor = MakeShared<FCurveEditor>();
	FCurveEditorInitParams InitParams;
	InitParams.ResolveCurveModelDelegate.BindStatic(&UE::CurveAssetEditor::ResolveCurveModel); // Needed for undo / redo
	CurveEditor->InitCurveEditor(InitParams);
	CurveEditor->GridLineLabelFormatXAttribute = LOCTEXT("GridXLabelFormat", "{0}");

	// Initialize our bounds at slightly larger than default to avoid clipping the tabs on the color widget.
	TUniquePtr<ICurveEditorBounds> EditorBounds = MakeUnique<FStaticCurveEditorBounds>();
	EditorBounds->SetInputBounds(-1.05, 1.05);
	CurveEditor->SetBounds(MoveTemp(EditorBounds));

	CurveEditorPanel = SNew(SCurveEditorPanel, CurveEditor.ToSharedRef())
		.TreeContent()
		[
			SNew(SCurveEditorTree, CurveEditor)
		];

	UCurveBase* Curve = Cast<UCurveBase>(GetEditingObject());
	if (Curve)
	{
		check(CurveEditor.IsValid());
		if (Curve->HasRichCurves())
		{
			for (const FRichCurveEditInfo& CurveData : Curve->GetCurves())
			{		
				TSharedPtr<FCurveAssetEditorTreeItem> TreeItem = MakeShared<FCurveAssetEditorTreeItem>(Curve, CurveData);

				// Add the channel to the tree-item and let it manage the lifecycle of the tree item.
				FCurveEditorTreeItem* NewItem = CurveEditor->AddTreeItem(FCurveEditorTreeItemID::Invalid());
				NewItem->SetStrongItem(TreeItem);
				
				// Pin all of the created curves by default for now so that they're visible when you open the
				// editor. Since there's only ever up to 4 channels we don't have to worry about overwhelming
				// amounts of curves.
				for (const FCurveModelID& CurveModel : NewItem->GetOrCreateCurves(CurveEditor.Get()))
				{
					CurveEditor->PinCurve(CurveModel);
				}
			}
		}
		else
		{
			UE_LOGF(LogTemp, Error, "Non-rich curves are not supported in the Curve Asset editor at this time.");
		}
	}

	

	TSharedRef<SDockTab> NewDockTab = SNew(SDockTab)
		.Label(FText::Format(LOCTEXT("CurveAssetEditorTitle", "{0} Curve Asset"), FText::FromString(GetTabPrefix())))
		.TabColorScale(GetTabColorScale())
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(0.0f)
			[
				CurveEditorPanel.ToSharedRef()
			]
		];

	// Insert a widget for editing the curve as a Color Gradient if it's a color curve we're editing.
	UCurveLinearColor* ColorCurve = Cast<UCurveLinearColor>(GetEditingObject());
	if (ColorCurve)
	{
		// We want to register an additional color editing view. This is effectively a "pinned" view who's visibility is controlled by the editor.
		TSharedRef<SColorGradientCurveEditorView> ColorGradientView = SNew(SColorGradientCurveEditorView, CurveEditor.ToSharedRef())
			.ViewMinInput(this, &FCurveAssetEditor::GetColorGradientViewMin)
			.ViewMaxInput(this, &FCurveAssetEditor::GetColorGradientViewMax)
			.IsEditingEnabled(true);

		ColorGradientView->GetGradientEditor()->SetCurveOwner(ColorCurve);
		CurveEditorPanel->AddView(ColorGradientView);
	}

	return NewDockTab;
}

TSharedRef<SDockTab> FCurveAssetEditor::SpawnTab_ColorCurveEditor(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == ColorCurveEditorTabId);


	TSharedRef<SDockTab> NewDockTab = SNew(SDockTab)
		.Label(LOCTEXT("ColorCurveEditor", "Color Curve Editor"))
		.TabColorScale(GetTabColorScale())
		[
			ColorCurveDetailsView.ToSharedRef()
		];

	return NewDockTab;
}

TSharedPtr<FExtender> FCurveAssetEditor::GetToolbarExtender()
{
	// Use the Curve Editor Panel's extenders which already has all of the icons listed in the right order.
	return CurveEditorPanel->GetToolbarExtender();
}

EOrientation FCurveAssetEditor::GetSnapLabelOrientation() const
{
	return FMultiBoxSettings::UseSmallToolBarIcons.Get()
		? EOrientation::Orient_Horizontal
		: EOrientation::Orient_Vertical;
}

TSharedRef<SWidget> FCurveAssetEditor::MakeCurveEditorCurveOptionsMenu()
{
	struct FExtrapolationMenus
	{
		static void MakePreInfinityExtrapSubMenu(FMenuBuilder& MenuBuilder)
		{
			MenuBuilder.BeginSection( "Pre-Infinity Extrapolation", LOCTEXT( "CurveEditorMenuPreInfinityExtrapHeader", "Extrapolation" ) );
			{
				MenuBuilder.AddMenuEntry( FCurveEditorCommands::Get().SetPreInfinityExtrapCycle);
				MenuBuilder.AddMenuEntry( FCurveEditorCommands::Get().SetPreInfinityExtrapCycleWithOffset);
				MenuBuilder.AddMenuEntry( FCurveEditorCommands::Get().SetPreInfinityExtrapOscillate);
				MenuBuilder.AddMenuEntry( FCurveEditorCommands::Get().SetPreInfinityExtrapLinear);
				MenuBuilder.AddMenuEntry( FCurveEditorCommands::Get().SetPreInfinityExtrapConstant);
			}
			MenuBuilder.EndSection();
		}

		static void MakePostInfinityExtrapSubMenu(FMenuBuilder& MenuBuilder)
		{
			MenuBuilder.BeginSection( "Post-Infinity Extrapolation", LOCTEXT( "CurveEditorMenuPostInfinityExtrapHeader", "Extrapolation" ) );
			{
				MenuBuilder.AddMenuEntry( FCurveEditorCommands::Get().SetPostInfinityExtrapCycle);
				MenuBuilder.AddMenuEntry( FCurveEditorCommands::Get().SetPostInfinityExtrapCycleWithOffset);
				MenuBuilder.AddMenuEntry( FCurveEditorCommands::Get().SetPostInfinityExtrapOscillate);
				MenuBuilder.AddMenuEntry( FCurveEditorCommands::Get().SetPostInfinityExtrapLinear);
				MenuBuilder.AddMenuEntry( FCurveEditorCommands::Get().SetPostInfinityExtrapConstant);
			}
			MenuBuilder.EndSection();
		}
	};

	FMenuBuilder MenuBuilder( true, CurveEditor->GetCommands());

	MenuBuilder.AddMenuEntry( FCurveEditorCommands::Get().BakeCurve);
	MenuBuilder.AddMenuEntry( FCurveEditorCommands::Get().ReduceCurve);

	MenuBuilder.AddSubMenu(
		LOCTEXT( "PreInfinitySubMenu", "Pre-Infinity" ),
		LOCTEXT( "PreInfinitySubMenuToolTip", "Pre-Infinity Extrapolation" ),
		FNewMenuDelegate::CreateStatic( &FExtrapolationMenus::MakePreInfinityExtrapSubMenu ) );

	MenuBuilder.AddSubMenu(
		LOCTEXT( "PostInfinitySubMenu", "Post-Infinity" ),
		LOCTEXT( "PostInfinitySubMenuToolTip", "Post-Infinity Extrapolation" ),
		FNewMenuDelegate::CreateStatic( &FExtrapolationMenus::MakePostInfinityExtrapSubMenu ) );
	
	return MenuBuilder.MakeWidget();
}

float FCurveAssetEditor::GetColorGradientViewMin() const
{
	double MinBounds, MaxBounds;
	CurveEditor->GetBounds().GetInputBounds(MinBounds, MaxBounds);

	return MinBounds;
}

float FCurveAssetEditor::GetColorGradientViewMax() const
{
	double MinBounds, MaxBounds;
	CurveEditor->GetBounds().GetInputBounds(MinBounds, MaxBounds);

	return MaxBounds;
}

#undef LOCTEXT_NAMESPACE
