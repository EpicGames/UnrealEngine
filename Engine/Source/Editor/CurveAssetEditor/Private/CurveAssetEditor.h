// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "ICurveAssetEditor.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/IToolkit.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"

class FCurveEditor;
class FExtender;
class FSpawnTabArgs;
class SCurveEditorPanel;
class SDockTab;
class SWidget;
class UCurveBase;

class FCurveAssetEditor :  public ICurveAssetEditor
{
public:

	FCurveAssetEditor()
	{}

	virtual ~FCurveAssetEditor() = default;

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	/**
	 * Edits the specified table
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	CurveToEdit				The curve to edit
	 */
	void InitCurveAssetEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UCurveBase* CurveToEdit );

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	TSharedPtr<FExtender> GetToolbarExtender();
	TSharedRef<SWidget> MakeCurveEditorCurveOptionsMenu();

private:

	/**	Spawns the tab with the curve asset inside */
	TSharedRef<SDockTab> SpawnTab_CurveAsset( const FSpawnTabArgs& Args );
	/**	Spawns the details panel for the color curve */
	TSharedRef<SDockTab> SpawnTab_ColorCurveEditor(const FSpawnTabArgs& Args);

	/** Get the orientation for the snap value controls. */
	EOrientation GetSnapLabelOrientation() const;

	/** Get the view min range from the Bounds to feed to the Color Gradient Editor when we're editing a color curve. */
	float GetColorGradientViewMin() const;
	/** Get the view max range from the Bounds to feed to the Color Gradient Editor when we're editing a color curve. */
	float GetColorGradientViewMax() const;

private:
	TSharedPtr<FCurveEditor> CurveEditor;
	TSharedPtr<SCurveEditorPanel> CurveEditorPanel;

	/**	The tab id for the curve asset tab */
	static const FName CurveTabId;
	/**	The tab id for the color curve editor tab */
	static const FName ColorCurveEditorTabId;

	/* Holds the details panel for the color curve */
	TSharedPtr<class IDetailsView> ColorCurveDetailsView;
};
