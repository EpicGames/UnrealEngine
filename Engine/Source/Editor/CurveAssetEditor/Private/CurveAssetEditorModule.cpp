// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveAssetEditorModule.h"

#include "CurveAssetEditor.h"
#include "CurveEditorCommands.h"
#include "HAL/Platform.h"
#include "Modules/ModuleManager.h"
#include "UObject/NameTypes.h"

class ICurveAssetEditor;
class IToolkitHost;
//#include "Toolkits/ToolkitManager.h"

IMPLEMENT_MODULE( FCurveAssetEditorModule, CurveAssetEditor );


const FName FCurveAssetEditorModule::CurveAssetEditorAppIdentifier( TEXT( "CurveAssetEditorApp" ) );

void FCurveAssetEditorModule::StartupModule()
{
	MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
	FCurveEditorCommands::Register();
}

void FCurveAssetEditorModule::ShutdownModule()
{
	MenuExtensibilityManager.Reset();
}

TSharedRef<ICurveAssetEditor> FCurveAssetEditorModule::CreateCurveAssetEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UCurveBase* CurveToEdit )
{
	TSharedRef< FCurveAssetEditor > NewCurveAssetEditor( new FCurveAssetEditor() );
	NewCurveAssetEditor->InitCurveAssetEditor( Mode, InitToolkitHost, CurveToEdit );
	return NewCurveAssetEditor;
}

