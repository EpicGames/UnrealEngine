// Copyright Epic Games, Inc. All Rights Reserved.

#include "SColorGradientCurveEditorView.h"

#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "SColorGradientEditor.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SCompoundWidget.h"

class FCurveEditor;

void SColorGradientCurveEditorView::Construct(const FArguments& InArgs, TSharedRef<FCurveEditor> InCurveEditor)
{
	bPinned = 1;
	bInteractive = 0;
	bAutoSize = 1;
	bAllowEmpty = 1;
	InitCurveEditorReference(InCurveEditor);

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.BorderBackgroundColor(FLinearColor(.8f, .8f, .8f, .60f))
		.Padding(1.0f)
		[
			SAssignNew(GradientViewer, SColorGradientEditor)
			.ViewMinInput(InArgs._ViewMinInput)
			.ViewMaxInput(InArgs._ViewMaxInput)
			.IsEditingEnabled(InArgs._IsEditingEnabled)
		]
	];
}

void SColorGradientCurveEditorView::CheckCacheAndInvalidateIfNeeded()
{
	// Always refresh for now
	// Could be improved by combining SCurveEditorView::CheckCacheAndInvalidateIfNeeded with a separate check for changes in the gradient stops
	RefreshRetainer();
}