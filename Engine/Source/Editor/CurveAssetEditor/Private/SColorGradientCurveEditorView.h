// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Attribute.h"
#include "SColorGradientEditor.h"
#include "SCurveEditorView.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FCurveEditor;

/**
* This wraps a SColorGradientEditor widget. The SColorGradientEditor widget doesn't
* know about Curve Models, but we need to show the SColorGradientEditor inside the normal
* curve stack of the new SCurveEditorPanel.
*
* To facilitate this, the FCurveAssetEditor creates an instance of this view and adds it as a
* pinned view to the SCurveEditorPanel it created. Then FCurveAssetEditor can directly set up the
* SColorGradientEditor (that this wraps) since it already knows about the FCurveOwnerInterface*.
* 
* Because of this, this view is tucked inside the CurveAssetEditor as a private class.
*/
class SColorGradientCurveEditorView : public SCurveEditorView
{
public:
	SLATE_BEGIN_ARGS(SColorGradientCurveEditorView)
	{}
	/** Min Bounds for the View (in Seconds) */
	SLATE_ATTRIBUTE(float, ViewMinInput)

	/** Max Bounds for the View (in Seconds) */
	SLATE_ATTRIBUTE(float, ViewMaxInput)

	/** Should the control be able to edit this curve or is editing disabled? */
	SLATE_ATTRIBUTE(bool, IsEditingEnabled)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FCurveEditor> InCurveEditor);

	TSharedRef<SColorGradientEditor> GetGradientEditor() const { return GradientViewer.ToSharedRef(); }

	virtual void CheckCacheAndInvalidateIfNeeded() override;
private:
	TSharedPtr<SColorGradientEditor> GradientViewer;
};
