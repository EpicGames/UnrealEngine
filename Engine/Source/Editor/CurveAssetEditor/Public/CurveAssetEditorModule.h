// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ICurveAssetEditor.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/IToolkit.h"
#include "Toolkits/IToolkitHost.h"

class FName;
class ICurveAssetEditor;
class UCurveBase;

/** DataTable Editor module */
class FCurveAssetEditorModule : public IModuleInterface,
	public IHasMenuExtensibility
{

public:
	// IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/**
	 * Creates an instance of table editor object.  Only virtual so that it can be called across the DLL boundary.
	 *
	 * @param	Mode					Mode that this editor should operate in
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	CurveToEdit				The Curve to start editing
	 *
	 * @return	Interface to the new curve asset editor
	 */
	virtual TSharedRef<ICurveAssetEditor> CreateCurveAssetEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UCurveBase* CurveToEdit );

	/** Gets the extensibility managers for outside entities to extend curve asset editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override {return MenuExtensibilityManager;}

	/** Curve Asset Editor app identifier string */
	static const FName CurveAssetEditorAppIdentifier;

private:
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
};


